{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Field System",
    "Description" : "Analytic Field",
    "Category": "Geometry",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : false,
    "CanContainContent" : true,
    "IsBetaVersion" : true,
    "Installed" : false,
    "Modules": [
    {
      "Name": "FieldSystemEditor",
      "Type": "Editor",
      "LoadingPhase": "Default"
    },
    ]
}
