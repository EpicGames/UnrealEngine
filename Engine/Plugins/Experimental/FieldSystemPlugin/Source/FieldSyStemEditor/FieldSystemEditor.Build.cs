// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class FieldSystemEditor : ModuleRules
	{
        public FieldSystemEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Slate",
                    "SlateCore",
                    "Engine",
					"EditorFramework",
                    "UnrealEd",
                    "PropertyEditor",
                    "RenderCore",
                    "RHI",
					"Chaos",
                    "FieldSystemEngine",
                    "RawMesh",
                    "AssetTools",
                    "AssetRegistry",
					"ToolMenus",
				}
				);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
