// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/ActorFactoryFieldSystem.h"
#include "Field/FieldSystemActor.h"
#include "Field/FieldSystemComponent.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "ActorFactoryFieldSystem"

DEFINE_LOG_CATEGORY_STATIC(AFFS_Log, Log, All);

/*-----------------------------------------------------------------------------
UActorFactoryFieldSystem
-----------------------------------------------------------------------------*/
UActorFactoryFieldSystem::UActorFactoryFieldSystem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("FieldSystemDisplayName", "FieldSystem");
	NewActorClass = AFieldSystemActor::StaticClass();
}

bool UActorFactoryFieldSystem::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(UFieldSystem::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoFieldSystemSpecified", "No FieldSystem mesh was specified.");
		return false;
	}

	return true;
}

void UActorFactoryFieldSystem::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UFieldSystem* FieldSystem = CastChecked<UFieldSystem>(Asset);
	AFieldSystemActor* NewFieldSystemActor = CastChecked<AFieldSystemActor>(NewActor);

	// Term Component
	 NewFieldSystemActor->GetFieldSystemComponent()->UnregisterComponent();

	// Change properties
	NewFieldSystemActor->GetFieldSystemComponent()->SetFieldSystem(FieldSystem);

	// Init Component
	NewFieldSystemActor->GetFieldSystemComponent()->RegisterComponent();
}

#undef LOCTEXT_NAMESPACE
