// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/AssetTypeActions_FieldSystem.h"

#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Field/FieldSystemAsset.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_FieldSystem::GetSupportedClass() const
{
	return UFieldSystem::StaticClass();
}

UThumbnailInfo* FAssetTypeActions_FieldSystem::GetThumbnailInfo(UObject* Asset) const
{
	UFieldSystem* FieldSystem = CastChecked<UFieldSystem>(Asset);
	return NewObject<USceneThumbnailInfo>(FieldSystem, NAME_None, RF_Transactional);
}

void FAssetTypeActions_FieldSystem::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	FAssetTypeActions_Base::GetActions(InObjects, Section);
}

void FAssetTypeActions_FieldSystem::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
}

#undef LOCTEXT_NAMESPACE
