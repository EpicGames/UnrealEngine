// Copyright Epic Games, Inc. All Rights Reserved.


#include "Field/FieldSystemEditorModule.h"

#include "Field/AssetTypeActions_FieldSystem.h"
#include "Field/FieldSystemEditorStyle.h"

TOptional<FFieldSystemEditorStyle> FFieldSystemEditorStyle::Singleton;

IMPLEMENT_MODULE( IFieldSystemEditorModule, FieldSystemEditor )

void IFieldSystemEditorModule::StartupModule()
{
	FFieldSystemEditorStyle::Get();

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	IAssetTools& AssetTools = AssetToolsModule.Get();
	AssetTypeActions_FieldSystem = new FAssetTypeActions_FieldSystem();
	AssetTools.RegisterAssetTypeActions(MakeShareable(AssetTypeActions_FieldSystem));

	if (GIsEditor && !IsRunningCommandlet())
	{
	}
}

void IFieldSystemEditorModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		IAssetTools& AssetTools = AssetToolsModule.Get();
		AssetTools.UnregisterAssetTypeActions(AssetTypeActions_FieldSystem->AsShared());
	}
}
