// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"

class FFieldSystemEditorStyle final : public FSlateStyleSet
{
public:
	FFieldSystemEditorStyle() : FSlateStyleSet("FieldSystemEditorStyle")
	{
		const FVector2D Icon16x16(16.f, 16.f);
		const FVector2D Icon64x64(64.f, 64.f);

#if !IS_MONOLITHIC
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/FieldSystemPlugin/Resources"));
#endif

		Set("ClassIcon.FieldSystem", new FSlateImageBrush(RootToContentDir(TEXT("FieldSystem_16x.png")), Icon16x16));
		Set("ClassThumbnail.FieldSystem", new FSlateImageBrush(RootToContentDir(TEXT("FieldSystem_64x.png")), Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FFieldSystemEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

public:

	static FFieldSystemEditorStyle& Get()
	{
		if (!Singleton.IsSet())
		{
			Singleton.Emplace();
		}
		return Singleton.GetValue();
	}

	static void Destroy()
	{
		Singleton.Reset();
	}

private:
	static TOptional<FFieldSystemEditorStyle> Singleton;
};

