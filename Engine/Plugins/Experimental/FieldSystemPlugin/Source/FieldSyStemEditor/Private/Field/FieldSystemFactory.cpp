// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystemFactory.h"

#include "Field/FieldSystemAsset.h"

#define LOCTEXT_NAMESPACE "FieldSystem"

/////////////////////////////////////////////////////
// FieldSystemFactory

UFieldSystemFactory::UFieldSystemFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UFieldSystem::StaticClass();
}

UFieldSystem* UFieldSystemFactory::StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UFieldSystem* System = static_cast<UFieldSystem*>(NewObject<UFieldSystem>(InParent, Class, Name, Flags | RF_Transactional | RF_Public | RF_Standalone));
	System->MarkPackageDirty();
	return System;
}

UObject* UFieldSystemFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{

	UFieldSystem* NewFieldSystem = StaticFactoryCreateNew(Class, InParent, Name, Flags, Context, Warn);
	NewFieldSystem->MarkPackageDirty();
	return NewFieldSystem;
}

#undef LOCTEXT_NAMESPACE



