// Copyright Epic Games, Inc. All Rights Reserved.

/** Factory which allows import of an FieldSystemAsset */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemAsset.h"

#include "FieldSystemFactory.generated.h"


/**
* Factory for Simple Cube
*/

UCLASS()
class FIELDSYSTEMEDITOR_API UFieldSystemFactory : public UFactory
{
    GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

	static UFieldSystem* StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn);

};


